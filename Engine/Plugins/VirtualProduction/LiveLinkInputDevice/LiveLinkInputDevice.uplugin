{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiveLinkInputDevice",
	"Description": "Live Link plugin for Unreal Engine Input Devices, i.e. Game Controllers. It uses the InputDevice system to query values and share state over LiveLink.",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"EnabledByDefault": false,
	"SupportedPrograms": ["LiveLinkHub"],
	"Modules": [
		{
			"Name": "LiveLinkInputDevice",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"ProgramAllowList": ["LiveLinkHub"],
			"PlatformAllowList": [
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "LiveLink",
			"Enabled": true
		}
	]

}
