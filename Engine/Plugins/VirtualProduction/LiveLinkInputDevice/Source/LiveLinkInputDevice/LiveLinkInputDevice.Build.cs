// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class LiveLinkInputDevice : ModuleRules
{
	public LiveLinkInputDevice(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"LiveLinkInterface",
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"CoreUObject",
				"Engine",
				"InputDevice",
				"Slate",
				"SlateCore"
			}
			);
	}
}
