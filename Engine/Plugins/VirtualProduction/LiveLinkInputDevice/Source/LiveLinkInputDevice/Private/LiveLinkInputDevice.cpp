// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkInputDevice.h"

#define LOCTEXT_NAMESPACE "LiveLinkInputDeviceModule"

DEFINE_LOG_CATEGORY(LogLiveLinkInputDevice);

void FLiveLinkInputDeviceModule::StartupModule()
{
}

void FLiveLinkInputDeviceModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FLiveLinkInputDeviceModule, LiveLinkInputDevice)