// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkInputDeviceMessageHandler.h"
#include "Roles/LiveLinkInputDeviceTypes.h"
#include "Misc/CoreMiscDefines.h"

#include "GenericPlatform/GenericApplicationMessageHandler.h"

namespace UE::InputDevice::Private
{

void SetValueWithKey(FName Key, float InValue, FLiveLinkGamepadInputDeviceFrameData& OutData)
{
	if (Key == FGamepadKeyNames::LeftAnalogX)
	{
		OutData.LeftAnalogX = InValue;
	}
	else
	if (Key == FGamepadKeyNames::LeftAnalogY)
	{
		OutData.LeftAnalogY = InValue;
	}
	else
	if (Key == FGamepadKeyNames::RightAnalogX)
	{
		OutData.RightAnalogX = InValue;
	}
	else
	if (Key == FGamepadKeyNames::RightAnalogY)
	{
		OutData.RightAnalogY = InValue;
	}
	else
	if (Key == FGamepadKeyNames::LeftTriggerAnalog)
	{
		OutData.LeftTriggerAnalog = InValue;
	}
	else
	if (Key == FGamepadKeyNames::RightTriggerAnalog)
	{
		OutData.RightTriggerAnalog = InValue;
	}
	else
	if (Key == FGamepadKeyNames::LeftThumb)
	{
		OutData.LeftThumb = InValue;
	}
	else
	if (Key == FGamepadKeyNames::RightThumb)
	{
		OutData.RightThumb = InValue;
	}
	else
	if (Key == FGamepadKeyNames::SpecialLeft)
	{
		OutData.SpecialLeft = InValue;
	}
	else
	if (Key == FGamepadKeyNames::SpecialLeft_X)
	{
		OutData.SpecialLeft_X = InValue;
	}
	else if (Key == FGamepadKeyNames::SpecialLeft_Y)
	{
		OutData.SpecialLeft_Y = InValue;
	}
	else if (Key == FGamepadKeyNames::SpecialRight)
	{
		OutData.SpecialRight = InValue;
	}
	else if (Key == FGamepadKeyNames::FaceButtonBottom)
	{
		OutData.FaceButtonBottom = InValue;
	}
	else if (Key == FGamepadKeyNames::FaceButtonRight)
	{
		OutData.FaceButtonRight = InValue;
	}
	else if (Key == FGamepadKeyNames::FaceButtonLeft)
	{
		OutData.FaceButtonLeft = InValue;
	}
	else if (Key == FGamepadKeyNames::FaceButtonTop)
	{
		OutData.FaceButtonTop = InValue;
	}
	else if (Key == FGamepadKeyNames::LeftShoulder)
	{
		OutData.LeftShoulder = InValue;
	}
	else if (Key == FGamepadKeyNames::RightShoulder)
	{
		OutData.RightShoulder = InValue;
	}
	else if (Key == FGamepadKeyNames::LeftTriggerThreshold)
	{
		OutData.LeftTriggerThreshold = InValue;
	}
	else if (Key == FGamepadKeyNames::RightTriggerThreshold)
	{
		OutData.RightTriggerThreshold = InValue;
	}
	else if (Key == FGamepadKeyNames::DPadUp)
	{
		OutData.DPadUp = InValue;
	}
	else if (Key == FGamepadKeyNames::DPadDown)
	{
		OutData.DPadDown = InValue;
	}
	else if (Key == FGamepadKeyNames::DPadRight)
	{
		OutData.DPadRight = InValue;
	}
	else if (Key == FGamepadKeyNames::DPadLeft)
	{
		OutData.DPadLeft = InValue;
	}
	else if (Key == FGamepadKeyNames::LeftStickUp)
	{
		OutData.LeftStickUp = InValue;
	}
	else if (Key == FGamepadKeyNames::LeftStickDown)
	{
		OutData.LeftStickDown = InValue;
	}
	else if (Key == FGamepadKeyNames::LeftStickRight)
	{
		OutData.LeftStickRight = InValue;
	}
	else if (Key == FGamepadKeyNames::LeftStickLeft)
	{
		OutData.LeftStickLeft = InValue;
	}
	else if (Key == FGamepadKeyNames::RightStickUp)
	{
		OutData.RightStickUp = InValue;
	}
	else if (Key == FGamepadKeyNames::RightStickDown)
	{
		OutData.RightStickDown = InValue;
	}
	else if (Key == FGamepadKeyNames::RightStickRight)
	{
		OutData.RightStickRight = InValue;
	}
	else if (Key == FGamepadKeyNames::RightStickLeft)
	{
		OutData.RightStickLeft = InValue;
	}
	else
	{
		ensureMsgf(false, TEXT("Unknown key value %s"), *Key.ToString());
	}
}
}

bool FLiveLinkInputDeviceMessageHandler::OnControllerAnalog(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, float AnalogValue)
{
	FLiveLinkGamepadInputDeviceFrameData& InputDeviceFrameData = CurrentFrameDataValues.FindOrAdd(InputDeviceId);
	UE::InputDevice::Private::SetValueWithKey(KeyName, AnalogValue, InputDeviceFrameData);
	return true;
}

bool FLiveLinkInputDeviceMessageHandler::OnControllerButtonPressed(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat)
{
	FLiveLinkGamepadInputDeviceFrameData& InputDeviceFrameData = CurrentFrameDataValues.FindOrAdd(InputDeviceId);
	UE::InputDevice::Private::SetValueWithKey(KeyName, 1, InputDeviceFrameData);
	return true;
}

bool FLiveLinkInputDeviceMessageHandler::OnControllerButtonReleased(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat)
{
	FLiveLinkGamepadInputDeviceFrameData& InputDeviceFrameData = CurrentFrameDataValues.FindOrAdd(InputDeviceId);
	UE::InputDevice::Private::SetValueWithKey(KeyName, 0, InputDeviceFrameData);
	return true;
}

FLiveLinkGamepadInputDeviceFrameData FLiveLinkInputDeviceMessageHandler::GetLatestValue(FInputDeviceId InDeviceId /*unused*/) const
{
	if (const TPair<FInputDeviceId, FLiveLinkGamepadInputDeviceFrameData>* Device = CurrentFrameDataValues.FindArbitraryElement())
	{
		return Device->Value;
	}
	return {};
}

TSet<FInputDeviceId> FLiveLinkInputDeviceMessageHandler::GetDeviceIds() const
{
	TSet<FInputDeviceId> Devices;
	CurrentFrameDataValues.GetKeys(Devices);
	return Devices;
}
