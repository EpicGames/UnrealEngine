// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "Roles/LiveLinkInputDeviceTypes.h"
#include "Containers/Map.h"
#include "Misc/CoreMiscDefines.h"

class FLiveLinkInputDeviceMessageHandler : public FGenericApplicationMessageHandler
{
public:
	virtual ~FLiveLinkInputDeviceMessageHandler() = default;

	virtual bool OnControllerAnalog(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, float AnalogValue) override;
	virtual bool OnControllerButtonPressed(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat) override;
	virtual bool OnControllerButtonReleased(FGamepadKeyNames::Type KeyName, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId, bool IsRepeat) override;

	/** For the given device id, which maps the the controller id, get the latest frame values for that device. */
	FLiveLinkGamepadInputDeviceFrameData GetLatestValue(FInputDeviceId InDeviceId) const;

	/** Return the current known list of devices */
	TSet<FInputDeviceId> GetDeviceIds() const;

private:
	TMap<FInputDeviceId, FLiveLinkGamepadInputDeviceFrameData> CurrentFrameDataValues;
};
