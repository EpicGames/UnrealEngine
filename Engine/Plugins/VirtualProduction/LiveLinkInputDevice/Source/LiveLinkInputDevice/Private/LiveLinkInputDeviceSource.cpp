// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkInputDeviceSource.h"

#include "Async/Async.h"
#include "CoreFwd.h"
#include "Engine/Engine.h"
#include "Features/IModularFeature.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "GenericPlatform/IInputInterface.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "IInputDeviceModule.h"
#include "InputCoreTypes.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkInputDeviceTypes.h"
#include "Roles/LiveLinkInputDeviceRole.h"

#include "ILiveLinkClient.h"
#include "LiveLinkInputDevice.h"
#include "LiveLinkInputDeviceSourceSettings.h"
#include "LiveLinkTypes.h"

#include "LiveLinkInputDeviceMessageHandler.h"

#define LOCTEXT_NAMESPACE "LiveLinkInputDeviceSource"

FLiveLinkInputDeviceSource::FLiveLinkInputDeviceSource(const FLiveLinkInputDeviceConnectionSettings& ConnectionSettings)
{
	UpdateGamepadState(false);
	SourceType = LOCTEXT("SourceType_InputDevice", "Input Device");
	SourceMachineName = LOCTEXT("LocalMachine", "Local");

	DeferredStartDelegateHandle = FCoreDelegates::OnEndFrame.AddRaw(this, &FLiveLinkInputDeviceSource::Start);
	IPlatformInputDeviceMapper::Get().GetOnInputDeviceConnectionChange().AddRaw(this, &FLiveLinkInputDeviceSource::HandleGamepadDeviceChange);
}

FLiveLinkInputDeviceSource::~FLiveLinkInputDeviceSource()
{
	if (DeferredStartDelegateHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(DeferredStartDelegateHandle);
	}
	IPlatformInputDeviceMapper::Get().GetOnInputDeviceConnectionChange().RemoveAll(this);
}

void FLiveLinkInputDeviceSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	Client = InClient;
	SourceGuid = InSourceGuid;
}

bool FLiveLinkInputDeviceSource::IsSourceStillValid() const
{
	const bool bIsSourceValid = !Stopping && (Thread != nullptr);
	return bIsSourceValid;
}

bool FLiveLinkInputDeviceSource::RequestSourceShutdown()
{
	Stop();

	return true;
}

// FRunnable interface
void FLiveLinkInputDeviceSource::Start()
{
	check(DeferredStartDelegateHandle.IsValid());

	FCoreDelegates::OnEndFrame.Remove(DeferredStartDelegateHandle);
	DeferredStartDelegateHandle.Reset();

	ThreadName = "LiveLinkInputDevice Receiver";
	ThreadName.AppendInt(FAsyncThreadIndex::GetNext());

	Thread.Reset(FRunnableThread::Create(this, *ThreadName, 128 * 1024, TPri_AboveNormal, FPlatformAffinity::GetPoolThreadMask()));
}

void FLiveLinkInputDeviceSource::Stop()
{
	Stopping = true;
}

void FLiveLinkInputDeviceSource::InitializeSettings(ULiveLinkSourceSettings* Settings)
{
	Settings->Mode = ELiveLinkSourceMode::Latest;
}

void FLiveLinkInputDeviceSource::LoadInputPlugins()
{
	MessageHandler = MakeShareable(new FLiveLinkInputDeviceMessageHandler());
	InputDevices.Reset();

	TArray<IInputDeviceModule*> PluginImplementations = IModularFeatures::Get().GetModularFeatureImplementations<IInputDeviceModule>( IInputDeviceModule::GetModularFeatureName() );
	for( auto InputPluginIt = PluginImplementations.CreateIterator(); InputPluginIt; ++InputPluginIt )
	{
		TSharedPtr<IInputDevice> Device = (*InputPluginIt)->CreateInputDevice(
			MessageHandler.ToSharedRef(),
			{
			.bInitAsPrimaryDevice = false,
		    });
		if (Device)
		{
			InputDevices.Add(Device);
		}
	}

	if (InputDevices.Num() > 0)
	{
		bHasLoadedInputPlugins = true;
	}
}

void FLiveLinkInputDeviceSource::UpdateGamepadState(const bool bInState)
{
	if (bInState)
	{
		SourceStatus = LOCTEXT("SourceStatus_Receiving", "Receiving");
	}
	else
	{
		SourceStatus = LOCTEXT("SourceStatus_Waiting", "Waiting for a device.");
	}
	bIsGamepadConnected = bInState;
}

bool FLiveLinkInputDeviceSource::TickInputDevices(const double InDeltaTime)
{
	static thread_local double ElapsedUpdate = 0;
	bool bGamepadState = false;

	const bool bShouldDoDeviceUpdate = bShouldRefreshDeviceList;
	for( int Index = 0; Index < InputDevices.Num(); Index++ )
	{
		if (bShouldDoDeviceUpdate)
		{
			static const FInputDeviceProperty RequestUpdateProp(TEXT("Request_Device_Update"));
			InputDevices[Index]->SetDeviceProperty(-1, &RequestUpdateProp);
		}
		InputDevices[Index]->Tick( InDeltaTime );
		InputDevices[Index]->SendControllerEvents();
		const bool bDeviceGamepadAttached = InputDevices[Index]->IsGamepadAttached();
		bGamepadState = bGamepadState || bDeviceGamepadAttached;
	}
	if (bShouldDoDeviceUpdate)
	{
		bShouldRefreshDeviceList = false;
	}

	ElapsedUpdate = ElapsedUpdate > 1 ? 0 : ElapsedUpdate + InDeltaTime;

	// Update our device for Live Link UI.
	//
	if (bIsGamepadConnected != bGamepadState)
	{
		UpdateGamepadState(bGamepadState);
	}

	return bGamepadState;
}

double FLiveLinkInputDeviceSource::DevicePollWaitTime() const
{
	const double UpdateRateInHz = LocalUpdateRateInHz > 0 ? (double) LocalUpdateRateInHz : 60.0;
	return 1.0 / UpdateRateInHz;
}

uint32 FLiveLinkInputDeviceSource::Run()
{
	// The amount of time we sleep on one iteration of this loop.  This is not const because we only update at the desired
	// update rate if we have input devices attached.  If no device is attached then we need to "wait" for a period of time
	// before reruning our loop.
	//
	double SleepDeltaTime = 1.0 / (double)LocalUpdateRateInHz;

	// Our wait time if no devices are initially connected.
	//
	const double WaitForDeviceTime = 1.0; // 1 Second.

	double LastUpdateTime = FPlatformTime::Seconds();
	while (!Stopping)
	{
		// Make sure we do not exceed our local update rate.
		ON_SCOPE_EXIT{
			FPlatformProcess::Sleep(SleepDeltaTime);
		};

		if (!GIsRunning)
		{
			continue;
		}

		if (!bHasLoadedInputPlugins)
		{
			LoadInputPlugins();
		}

		// Poll externally-implemented devices
		//
		const double CurrentTime = FPlatformTime::Seconds();
		TickInputDevices(CurrentTime - LastUpdateTime);;
		LastUpdateTime = CurrentTime;

		TSet<FInputDeviceId> Devices = MessageHandler->GetDeviceIds();

		if (Devices.Num() == 0)
		{
			// If no devices then we throttle back our update loop and wait for a device to come online.
			SleepDeltaTime = WaitForDeviceTime;
			continue;
		}

		for (FInputDeviceId Id : Devices)
		{
			if (TOptional<FLiveLinkStaticDataStruct> StaticData = PollGamepadStaticData(Id))
			{
				Send(Id, MoveTemp(*StaticData));
			}
			else if (TOptional<FLiveLinkFrameDataStruct> FrameData = PollGamepadFrameData(Id))
			{
				Send(Id, MoveTemp(*FrameData));
			}
		}

		SleepDeltaTime = DevicePollWaitTime();
	}
	
	return 0;
}

TOptional<FLiveLinkStaticDataStruct> FLiveLinkInputDeviceSource::PollGamepadStaticData(FInputDeviceId InDeviceId)
{
	if (Client)
	{
		if (!EncounteredSubjects.Contains(InDeviceId))
		{
			EncounteredSubjects.Add(InDeviceId);
			FLiveLinkStaticDataStruct StaticData(FLiveLinkGamepadInputDeviceStaticData::StaticStruct());
			return StaticData;
		}
	}

	return {};
}

TOptional<FLiveLinkFrameDataStruct> FLiveLinkInputDeviceSource::PollGamepadFrameData(FInputDeviceId InDeviceId)
{
	if (Client)
	{
		FLiveLinkFrameDataStruct BaseFrameData(FLiveLinkGamepadInputDeviceFrameData::StaticStruct());
		FLiveLinkGamepadInputDeviceFrameData* FrameData = BaseFrameData.Cast<FLiveLinkGamepadInputDeviceFrameData>();

		*FrameData = MessageHandler->GetLatestValue({});
		FrameData->WorldTime = FPlatformTime::Seconds();
		const TOptional<FQualifiedFrameTime> CurrentFrameTime = FApp::GetCurrentFrameTime();
		if (CurrentFrameTime)
		{
			FrameData->MetaData.SceneTime = *CurrentFrameTime;
		}

		return BaseFrameData;
	}
	return {};
}

void FLiveLinkInputDeviceSource::Send(FInputDeviceId DeviceId, FLiveLinkStaticDataStruct StaticDataToSend)
{
	if (Stopping || Client == nullptr)
	{
		return;
	}
	FNameBuilder Builder("Gamepad_");
	Builder.Append(FString::FromInt(DeviceId.GetId()));

	FName SubjectName = *Builder;
	Client->PushSubjectStaticData_AnyThread({ SourceGuid, SubjectName }, ULiveLinkInputDeviceRole::StaticClass(), MoveTemp(StaticDataToSend));
}

void FLiveLinkInputDeviceSource::Send(FInputDeviceId DeviceId, FLiveLinkFrameDataStruct FrameDataToSend)
{
	if (Stopping || Client == nullptr)
	{
		return;
	}
	FNameBuilder Builder("Gamepad_");
	Builder.Append(FString::FromInt(DeviceId.GetId()));
	FName SubjectName = *Builder;
	Client->PushSubjectFrameData_AnyThread({ SourceGuid, SubjectName }, MoveTemp(FrameDataToSend));
}

void FLiveLinkInputDeviceSource::HandleGamepadDeviceChange(EInputDeviceConnectionState /* unused */, FPlatformUserId /* unused  */, FInputDeviceId /* unused */)
{
    bShouldRefreshDeviceList = true;
}


#undef LOCTEXT_NAMESPACE
