// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkInputDeviceSourceFactory.h"

#include "Features/IModularFeatures.h"
#include "ILiveLinkClient.h"
#include "LiveLinkInputDeviceSource.h"
#include "LiveLinkInputDeviceSourceSettings.h"
#include "SLiveLinkInputDeviceSourceFactory.h"

#define LOCTEXT_NAMESPACE "LiveLinkInputDeviceSourceFactory"

FText ULiveLinkInputDeviceSourceFactory::GetSourceDisplayName() const
{
	return LOCTEXT("SourceDisplayName", "LiveLinkInputDevice Source");	
}

FText ULiveLinkInputDeviceSourceFactory::GetSourceTooltip() const
{
	return LOCTEXT("SourceTooltip", "Allows creation of LiveLink sources using the Unreal Engine input device system. Only gamepad controllers are currently supported.");
}

TSharedPtr<SWidget> ULiveLinkInputDeviceSourceFactory::BuildCreationPanel(FOnLiveLinkSourceCreated InOnLiveLinkSourceCreated) const
{
	return SNew(SLiveLinkInputDeviceSourceFactory)
		.OnConnectionSettingsAccepted(FOnLiveLinkInputDeviceConnectionSettingsAccepted::CreateUObject(this, &ULiveLinkInputDeviceSourceFactory::CreateSourceFromSettings, InOnLiveLinkSourceCreated));
}

TSharedPtr<ILiveLinkSource> ULiveLinkInputDeviceSourceFactory::CreateSource(const FString& ConnectionString) const
{
	FLiveLinkInputDeviceConnectionSettings ConnectionSettings;
	if (!ConnectionString.IsEmpty())
	{
		FLiveLinkInputDeviceConnectionSettings::StaticStruct()->ImportText(*ConnectionString, &ConnectionSettings, nullptr, PPF_None, GLog, TEXT("ULiveLinkInputDeviceSourceFactory"));
	}
	return MakeShared<FLiveLinkInputDeviceSource>(ConnectionSettings);
}

void ULiveLinkInputDeviceSourceFactory::CreateSourceFromSettings(FLiveLinkInputDeviceConnectionSettings InConnectionSettings, FOnLiveLinkSourceCreated OnSourceCreated) const
{
	FString ConnectionString;
	FLiveLinkInputDeviceConnectionSettings::StaticStruct()->ExportText(ConnectionString, &InConnectionSettings, nullptr, nullptr, PPF_None, nullptr);

	TSharedPtr<FLiveLinkInputDeviceSource> SharedPtr = MakeShared<FLiveLinkInputDeviceSource>(InConnectionSettings);
	OnSourceCreated.ExecuteIfBound(SharedPtr, MoveTemp(ConnectionString));
}

#undef LOCTEXT_NAMESPACE
