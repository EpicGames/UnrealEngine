// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkInputDeviceConnectionSettings.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#if WITH_EDITOR
#include "IStructureDetailsView.h"
#endif //WITH_EDITOR

#include "Input/Reply.h"

struct FLiveLinkInputDeviceConnectionSettings;

DECLARE_DELEGATE_OneParam(FOnLiveLinkInputDeviceConnectionSettingsAccepted, FLiveLinkInputDeviceConnectionSettings);

class SLiveLinkInputDeviceSourceFactory : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkInputDeviceSourceFactory)
	{}
		SLATE_EVENT(FOnLiveLinkInputDeviceConnectionSettingsAccepted, OnConnectionSettingsAccepted)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);


private:
	FLiveLinkInputDeviceConnectionSettings ConnectionSettings;

#if WITH_EDITOR
	TSharedPtr<FStructOnScope> StructOnScope;
	TSharedPtr<IStructureDetailsView> StructureDetailsView;
#endif //WITH_EDITOR

	FReply OnSettingsAccepted();
	FOnLiveLinkInputDeviceConnectionSettingsAccepted OnConnectionSettingsAccepted;
};