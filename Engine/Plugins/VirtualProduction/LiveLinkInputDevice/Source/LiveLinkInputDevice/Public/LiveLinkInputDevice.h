// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkInputDevice, Log, All);

/**
 * This module provides access to a restricted set of the InputDevice system over Live Link without using the Game Input Device Subsystem.
 *
 * This allows us to poll / query the input devices from another thread for recording and animating purposes.
 * */
class FLiveLinkInputDeviceModule : public IModuleInterface
{
public:

	static FLiveLinkInputDeviceModule& Get()
	{
		return FModuleManager::Get().LoadModuleChecked<FLiveLinkInputDeviceModule>(TEXT("LiveLinkInputDevice"));
	}

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
