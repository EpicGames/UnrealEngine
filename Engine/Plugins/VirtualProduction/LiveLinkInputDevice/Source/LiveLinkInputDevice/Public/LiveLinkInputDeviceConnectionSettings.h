// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkInputDeviceConnectionSettings.generated.h"

USTRUCT()
struct LIVELINKINPUTDEVICE_API FLiveLinkInputDeviceConnectionSettings
{
	GENERATED_BODY()
};
