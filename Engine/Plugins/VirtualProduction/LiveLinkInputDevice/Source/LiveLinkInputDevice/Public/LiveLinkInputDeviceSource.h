// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkSource.h"
#include "LiveLinkInputDeviceConnectionSettings.h"
#include "LiveLinkInputDeviceMessageHandler.h"
#include "LiveLinkInputDeviceSourceSettings.h"
#include "Misc/CoreMiscDefines.h"

#include "Roles/LiveLinkInputDeviceTypes.h"
#include "Roles/LiveLinkTransformTypes.h"

#include "Delegates/IDelegateInstance.h"
#include "MessageEndpoint.h"
#include "IMessageContext.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/Runnable.h"

struct ULiveLinkInputDeviceSettings;

class IInputDevice;
class ILiveLinkClient;


class LIVELINKINPUTDEVICE_API FLiveLinkInputDeviceSource : public ILiveLinkSource, public FRunnable, public TSharedFromThis<FLiveLinkInputDeviceSource>
{
public:

	FLiveLinkInputDeviceSource(const FLiveLinkInputDeviceConnectionSettings& ConnectionSettings);

	virtual ~FLiveLinkInputDeviceSource();

	// Begin ILiveLinkSource Interface
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	virtual void InitializeSettings(ULiveLinkSourceSettings* Settings) override;
	virtual bool IsSourceStillValid() const override;

	virtual bool RequestSourceShutdown() override;

	virtual FText GetSourceType() const override { return SourceType; };
	virtual FText GetSourceMachineName() const override { return SourceMachineName; }
	virtual FText GetSourceStatus() const override { return SourceStatus; }

	virtual TSubclassOf<ULiveLinkSourceSettings> GetSettingsClass() const override { return ULiveLinkInputDeviceSourceSettings::StaticClass(); }
	// End ILiveLinkSource Interface

	// Begin FRunnable Interface
	virtual bool Init() override { return true; }
	virtual uint32 Run() override;
	void Start();
	virtual void Stop() override;
	virtual void Exit() override { }
	// End FRunnable Interface

private:
	TOptional<FLiveLinkStaticDataStruct> PollGamepadStaticData(FInputDeviceId SubjectName);
	TOptional<FLiveLinkFrameDataStruct>  PollGamepadFrameData(FInputDeviceId SubjectName);

	void Send(FInputDeviceId DeviceId, FLiveLinkStaticDataStruct FrameDataToSend);
	void Send(FInputDeviceId DeviceId, FLiveLinkFrameDataStruct StaticDataToSend);

	/** Ticks each input device and calls SendControllerEvents to update the device with our message handler. */
	bool TickInputDevices(const double InCurrentTime);

	/** Time in seconds to wait before polling the device values again. */
	double DevicePollWaitTime() const;

private:
	/** This delegate is invoked when a gamepad device is added or removed from the application. */
	void HandleGamepadDeviceChange(EInputDeviceConnectionState NewConnectionState, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId);

	/** Update the our known gamepad state so that it can be reflected back to the LiveLink UI. */
	void UpdateGamepadState(const bool bInState);

	/** Create load all input devices by calling CreateInputDevice. */
	void LoadInputPlugins();

	ILiveLinkClient* Client;

	// Our identifier in LiveLink
	FGuid SourceGuid;

	// Text objects that are reflected back in the Live Link UI.
	FText SourceType;
	FText SourceMachineName;
	FText SourceStatus;
	
	// Threadsafe Bool for terminating the main thread loop
	FThreadSafeBool Stopping;
	
	// Thread to run updates on
	TUniquePtr<FRunnableThread> Thread;

	// Name of the updates thread
	FString ThreadName;

	// List of subjects we've already encountered
	TSet<FInputDeviceId> EncounteredSubjects;

	// Deferred start delegate handle
	FDelegateHandle DeferredStartDelegateHandle;

	// Rate at which we poll the input devices.
	uint32 LocalUpdateRateInHz = 100;

	// Our local message handler.
	TSharedPtr<class FLiveLinkInputDeviceMessageHandler> MessageHandler;

	// List of known input devices registered with this source.
	TArray<TSharedPtr<IInputDevice>> InputDevices;

	// Bool to track if we have loaded our input plugins.  This should only be done once by querying the input device modular feature.
	bool bHasLoadedInputPlugins = false;

	// Indicates if we have ANY gamepads connected and thus are an active device for Live Link.
	bool bIsGamepadConnected = false;

	std::atomic<bool> bShouldRefreshDeviceList = false;
};
