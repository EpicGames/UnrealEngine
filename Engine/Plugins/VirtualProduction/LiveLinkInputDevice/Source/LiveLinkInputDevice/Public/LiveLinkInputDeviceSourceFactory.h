// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSourceFactory.h"
#include "LiveLinkInputDeviceSource.h"
#include "LiveLinkInputDeviceSourceFactory.generated.h"

UCLASS()
class LIVELINKINPUTDEVICE_API ULiveLinkInputDeviceSourceFactory : public ULiveLinkSourceFactory
{
public:
	GENERATED_BODY()

	virtual FText GetSourceDisplayName() const override;
	virtual FText GetSourceTooltip() const override;

	virtual EMenuType GetMenuType() const override { return EMenuType::MenuEntry; }
	virtual TSharedPtr<SWidget> BuildCreationPanel(FOnLiveLinkSourceCreated OnLiveLinkSourceCreated) const override;
	virtual TSharedPtr<ILiveLinkSource> CreateSource(const FString& ConnectionString) const override;

private:
	void CreateSourceFromSettings(FLiveLinkInputDeviceConnectionSettings ConnectionSettings, FOnLiveLinkSourceCreated OnSourceCreated) const;
};