// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkSourceSettings.h"
#include "LiveLinkInputDeviceSourceSettings.generated.h"

UCLASS()
class LIVELINKINPUTDEVICE_API ULiveLinkInputDeviceSourceSettings : public ULiveLinkSourceSettings
{
	GENERATED_BODY()
};
